{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Runtime Tests",
	"Description": "Automated tests for runtime code used in both editor and cooked builds",
	"Category": "Testing",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "RuntimeTests",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		}
	],
	"Plugins": [
		{
			"Name": "AsyncMessageSystem",
			"Enabled": true
		}
	]
}