// Copyright Epic Games, Inc. All Rights Reserved.

#include "EngineRuntimeTests.h"
#include "Components/BillboardComponent.h"
#include "Engine/Engine.h"
#include "Engine/GameInstance.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "EngineUtils.h"
#include "Math/RandomStream.h"
#include "Misc/AutomationTest.h"
#include "Stats/StatsMisc.h"
#include "Containers/Ticker.h"
#include "Tests/AutomationCommon.h"
#include "TimerManager.h"
#include "Tickable.h"
#include "AsyncMessageHandle.h"
#include "AsyncMessageSystemBase.h"
#include "AsyncMessageWorldSubsystem.h"
#include "AsyncGameplayMessageSystem.h"
#include "NativeGameplayTags.h"
#include "Tasks/Task.h"
#include "MassEntityManager.h"
#include "MassEntityUtils.h"
#include "MassProcessingTypes.h"
#include "MassExecutor.h"
#include "MassExecutionContext.h"
#include "MassProcessingPhaseManager.h"
#include "TaskSyncManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EngineRuntimeTests)

int32 AEngineTestTickActor::CurrentTickOrder = 0;

AEngineTestTickActor::AEngineTestTickActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SpriteComponent = CreateDefaultSubobject<UBillboardComponent>(TEXT("Sprite"));
	if (SpriteComponent)
	{
		SpriteComponent->bHiddenInGame = true;
		RootComponent = SpriteComponent;
	}

	PrimaryActorTick.TickGroup = TG_PrePhysics;
	PrimaryActorTick.bCanEverTick = true;

	ResetState();
}

void AEngineTestTickActor::ResetState()
{
	TickCount = 0;
	TickOrder = 0;
	bShouldIncrementTickCount = true;
	bShouldDoMath = true;
	MathCounter = 0.0f;
	MathIncrement = 0.01f;
	MathLimit = 1.0f;
}

void AEngineTestTickActor::DoTick()
{
	if (bShouldIncrementTickCount)
	{
		TickCount++;
	}

	if (bShouldDoMath && MathIncrement > 0.0f && MathLimit > 0.0f)
	{
		MathCounter = 0.0f;
		while (MathCounter < MathLimit)
		{
			MathCounter += MathIncrement;
		}
	}

	TickOrder = CurrentTickOrder++;
}

void AEngineTestTickActor::VirtualTick()
{
	DoTick();
}

void AEngineTestTickActor::Tick(float DeltaSeconds)
{
	Super::Tick(DeltaSeconds);

	DoTick();
}

// Mass processor for tick testing
UEngineTickTestProcessor::UEngineTickTestProcessor()
	: EntityQuery(*this)
{
#if WITH_EDITORONLY_DATA
	bCanShowUpInSettings = false;
#endif // WITH_EDITORONLY_DATA
	bAutoRegisterWithProcessingPhases = false;
	bRequiresGameThreadExecution = false; // Comment out to test async
	ExecutionFlags = int32(EProcessorExecutionFlags::All);

	ForEachEntityChunkExecutionFunction = [](FMassExecutionContext& Context) {};
}

void UEngineTickTestProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	EntityQuery.ForEachEntityChunk(Context, ForEachEntityChunkExecutionFunction);
}

FGraphEventRef UEngineTickTestProcessor::DispatchProcessorTasks(const TSharedPtr<FMassEntityManager>& EntityManager, FMassExecutionContext& ExecutionContext, const FGraphEventArray& Prerequisites)
{
	FGraphEventRef ReturnRef = Super::DispatchProcessorTasks(EntityManager, ExecutionContext, Prerequisites);

	if (!SyncPointName.IsNone())
	{
		UE::Tick::FTaskSyncManager* TaskManager = UE::Tick::FTaskSyncManager::Get();
		if (TaskManager)
		{
			UE::Tick::FSyncPointId SyncPoint = TaskManager->FindSyncPoint(EntityManager->GetWorld(), SyncPointName);
			TaskManager->TriggerSyncPointAfterEvent(SyncPoint, ReturnRef);
		}
	}

	return ReturnRef;
}

#if WITH_AUTOMATION_WORKER

FEngineTickTestBase::FEngineTickTestBase(const FString& InName, const bool bInComplexTask)
	: FAutomationTestBase(InName, bInComplexTask)
{
}

FEngineTickTestBase::~FEngineTickTestBase()
{
	// The unique ptr takes care of destroying the world
}

UWorld* FEngineTickTestBase::GetTestWorld() const
{
	if (WorldWrapper.IsValid())
	{
		return WorldWrapper->GetTestWorld();
	}
	return nullptr;
}

bool FEngineTickTestBase::CreateTestWorld()
{
	if (!TestTrue(TEXT("TestWorld already exists in CreateTestWorld!"), GetTestWorld() == nullptr))
	{
		return false;
	}

	if (!WorldWrapper.IsValid())
	{
		WorldWrapper = TUniquePtr<FTestWorldWrapper>(new FTestWorldWrapper());
	}

	return WorldWrapper->CreateTestWorld(EWorldType::Game);
}

bool FEngineTickTestBase::CreateTestActors(int32 ActorCount, TSubclassOf<AEngineTestTickActor> ActorClass)
{
	UWorld* TestWorld = GetTestWorld();
	if (!TestNotNull(TEXT("TestWorld does not exist in CreateTestActors!"), TestWorld))
	{
		return false;
	}

	for (int32 i = 0; i < ActorCount; i++)
	{
		AEngineTestTickActor* TickActor = Cast<AEngineTestTickActor>(TestWorld->SpawnActor(ActorClass.Get()));
		if (!TestNotNull(TEXT("CreateTestActors failed to spawn actor!"), TickActor))
		{
			return false;
		}
		TickActor->ResetState();
		TestActors.Add(TickActor);
	}

	return true;
}

bool FEngineTickTestBase::BeginPlayInTestWorld()
{
	UWorld* TestWorld = GetTestWorld();
	if (!TestNotNull(TEXT("TestWorld does not exist in BeginPlayInTestWorld!"), TestWorld))
	{
		return false;
	}

	return WorldWrapper->BeginPlayInTestWorld();
}

bool FEngineTickTestBase::TickTestWorld(float DeltaTime)
{
	UWorld* TestWorld = GetTestWorld();
	if (!TestNotNull(TEXT("TestWorld does not exist in TickTestWorld!"), TestWorld))
	{
		return false;
	}

	AEngineTestTickActor::CurrentTickOrder = 1;
	return WorldWrapper->TickTestWorld(DeltaTime);
}

bool FEngineTickTestBase::ResetTestActors()
{
	for (AEngineTestTickActor* TestActor : TestActors)
	{
		TestActor->ResetState();
	}
	return true;
}

bool FEngineTickTestBase::CheckTickCount(const TCHAR* TickTestName, int32 TickCount)
{
	for (AEngineTestTickActor* TestActor : TestActors)
	{
		if (!TestEqual(TickTestName, TestActor->TickCount, TickCount))
		{
			return false;
		}
	}
	return true;
}

bool FEngineTickTestBase::DestroyAllTestActors()
{
	UWorld* TestWorld = GetTestWorld();
	if (!TestNotNull(TEXT("TestWorld does not exist in CreateTestActors!"), TestWorld))
	{
		return false;
	}

	for (AEngineTestTickActor* TestActor : TestActors)
	{
		TestActor->Destroy();
	}

	TestActors.Empty();
		
	return true;
}

bool FEngineTickTestBase::DestroyTestWorld()
{
	if (WorldWrapper.IsValid())
	{
		DestroyAllTestActors();
		return WorldWrapper->DestroyTestWorld(true);
	}
	return false;
}

bool FEngineTickTestBase::ReportAnyErrors()
{
	if (WorldWrapper.IsValid())
	{
		WorldWrapper->ForwardErrorMessages(this);
	}
	return HasAnyErrors();
}


// Emulate an efficiently registered tick with caching
struct FEngineTestTickActorTickableFast : FTickableGameObject
{
	// Not safe to use outside these tests
	AEngineTestTickActor* TickActor = nullptr;
	UWorld* CachedWorld = nullptr;

	FEngineTestTickActorTickableFast(AEngineTestTickActor* InTickActor)
		: TickActor(InTickActor)
	{
		CachedWorld = InTickActor->GetWorld();
	}

	virtual void Tick(float DeltaTime) override { TickActor->DoTick(); }
	virtual UWorld* GetTickableGameObjectWorld() const override { return CachedWorld; }
	virtual bool IsTickableWhenPaused() const override { return false; }
	virtual bool IsTickableInEditor() const override { return false; }
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Always; }
	virtual TStatId GetStatId() const override { return TStatId(); }
};

// Emulates a safer and slower setup
struct FEngineTestTickActorTickableSlow : FTickableGameObject
{
	// Not safe to use outside these tests
	AEngineTestTickActor* TickActor = nullptr;

	FEngineTestTickActorTickableSlow(AEngineTestTickActor* InTickActor)
		: TickActor(InTickActor)
	{
	}

	virtual void Tick(float DeltaTime) override { TickActor->VirtualTick(); }
	virtual UWorld* GetTickableGameObjectWorld() const override { return TickActor->GetWorld(); }
	virtual bool IsTickableWhenPaused() const override { return false; }
	virtual bool IsTickableInEditor() const override { return false; }
	virtual bool IsAllowedToTick() const { return IsValid(TickActor) && IsValid(TickActor->GetOuter()); }
	virtual bool IsTickable() const { return TickActor->bShouldIncrementTickCount; }
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Conditional; }
	virtual TStatId GetStatId() const override { RETURN_QUICK_DECLARE_CYCLE_STAT(FEngineTestTickActorTickableSlow, STATGROUP_Tickables); }
};


#define LOG_SCOPE_TIME(x) \
	TRACE_CPUPROFILER_EVENT_SCOPE(x); \
	FScopeLogTime LogTimePtr(TEXT(#x), nullptr, FScopeLogTime::ScopeLog_Milliseconds)

// Ensures that manually ticking a world works correctly
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FBasicTickTest, FEngineTickTestBase, "System.Engine.Tick.BasicTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ServerContext | EAutomationTestFlags::EngineFilter)
bool FBasicTickTest::RunTest(const FString& Parameters)
{
	int32 ActorCount = 10;
	int32 TickCount = 10;
	float DeltaTime = 0.01f;

	if (!CreateTestWorld())
	{
		return false;
	}

	bool bSuccess = true;

	bSuccess &= CreateTestActors(ActorCount, AEngineTestTickActor::StaticClass());
	bSuccess &= BeginPlayInTestWorld();
	 
	if (bSuccess)
	{
		for (int32 i = 0; i < TickCount; i++)
		{
			TickTestWorld(DeltaTime);
		}

		CheckTickCount(TEXT("TickCount"), TickCount);
	}

	// Always reset test world
	bSuccess &= DestroyTestWorld();

	return bSuccess && !ReportAnyErrors();
}

// Verify different methods of ordering ticks
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FOrderTickTest, FEngineTickTestBase, "System.Engine.Tick.OrderTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ServerContext | EAutomationTestFlags::EngineFilter)
bool FOrderTickTest::RunTest(const FString& Parameters)
{
	float DeltaTime = 0.01f;
	int32 ActorCount = 1000;

	if (!CreateTestWorld())
	{
		return false;
	}

	bool bSuccess = true;

	bSuccess &= CreateTestActors(ActorCount, AEngineTestTickActor::StaticClass());
	bSuccess &= BeginPlayInTestWorld();

	if (bSuccess)
	{
		check(TestActors.Num() == ActorCount);

		// Semirandom numbers, generally tick happens based on order of spawn but that is not guaranteed
		AEngineTestTickActor* HighPriority = TestActors[12];
		AEngineTestTickActor* HighPrereq = TestActors[18];
		AEngineTestTickActor* PostPhysics = TestActors[2];
		AEngineTestTickActor* PostPhysicsDep = TestActors[75];
		AEngineTestTickActor* PosyPhysicsDep2 = TestActors[45];
		AEngineTestTickActor* TickInterval = TestActors[32];
		AEngineTestTickActor* TickIntervalDep = TestActors[23];

		HighPriority->PrimaryActorTick.SetPriorityIncludingPrerequisites(true);
		PostPhysics->PrimaryActorTick.TickGroup = TG_PostPhysics;

		ResetTestActors();
		TickTestWorld(DeltaTime);

		TestEqual(TEXT("HighPriority tickorder"), HighPriority->TickOrder, 1);
		TestEqual(TEXT("PostPhysics tickorder"), PostPhysics->TickOrder, ActorCount);

		HighPriority->AddTickPrerequisiteActor(HighPrereq);

		// This has to be refreshed the tick prereq is set right now, comment out to verify
		HighPriority->PrimaryActorTick.SetPriorityIncludingPrerequisites(false);
		HighPriority->PrimaryActorTick.SetPriorityIncludingPrerequisites(true);

		// Test dependency group demoting
		PostPhysicsDep->AddTickPrerequisiteActor(PostPhysics);

		ResetTestActors();
		TickTestWorld(DeltaTime);

		TestEqual(TEXT("HighPrereq tickorder"), HighPrereq->TickOrder, 1);
		TestEqual(TEXT("HighPriority tickorder"), HighPriority->TickOrder, 2);
		TestEqual(TEXT("PostPhysicsDep tickorder"), PostPhysicsDep->TickOrder, ActorCount);


		// Uncomment to test circular reference, which throws off ordering
		// PostPhysics->AddTickPrerequisiteActor(PosyPhysicsDep2);
		PosyPhysicsDep2->AddTickPrerequisiteActor(PostPhysicsDep);

		// Test tick interval, it will be run the first tick but not the second
		TickInterval->SetActorTickInterval(0.5f);
		TickInterval->PrimaryActorTick.TickGroup = TG_PostUpdateWork;

		// The dependency will be respected the first time, but not the second
		TickIntervalDep->AddTickPrerequisiteActor(TickInterval);

		ResetTestActors();
		TickTestWorld(DeltaTime);

		TestEqual(TEXT("TickInterval count"), TickInterval->TickCount, 1);
		TestEqual(TEXT("TickIntervalDep tickorder"), TickIntervalDep->TickOrder, ActorCount); // This will be last because dependency is respected

		TickTestWorld(DeltaTime);

		TestEqual(TEXT("TickInterval count"), TickInterval->TickCount, 1); // This was skipped by second tick
		TestEqual(TEXT("TickIntervalDep count"), TickIntervalDep->TickCount, 2);
		TestEqual(TEXT("PosyPhysicsDep2 tickorder"), PosyPhysicsDep2->TickOrder, ActorCount - 1); // TickInterval is skipped on the second frame so this is last of 99

		TestEqual(TEXT("HighPrereq tickorder"), HighPrereq->TickOrder, 1);
		TestEqual(TEXT("HighPriority tickorder"), HighPriority->TickOrder, 2);
	}

	// Always reset test world
	bSuccess &= DestroyTestWorld();

	return bSuccess && !ReportAnyErrors();
}

UE_DEFINE_GAMEPLAY_TAG_COMMENT(TickTestMessageTag, "AsyncMessages.Internal.test.TickEvent", "Tag for testing async message tick event");

// Simple test message system that is executed manually
class FTestMessageSystem : public FAsyncMessageSystemBase
{
protected:
	virtual void Startup_Impl() override { }
	virtual void Shutdown_Impl() override { }
	virtual void PostQueueMessage(const FAsyncMessageId MessageId, const TArray<FAsyncMessageBindingOptions>& OptionsBoundTo) override { }
};

static TSharedRef<FMassProcessingPhaseManager> InitializeMassProcessing(FMassEntityManager* EntityManager, UMassProcessor* Processor)
{
	TSharedRef<FMassProcessingPhaseManager> PhaseManager = MakeShared<FMassProcessingPhaseManager>();

	FMassProcessingPhaseConfig PhasesConfig[int(EMassProcessingPhase::MAX)];
	PhaseManager->Initialize(*EntityManager->GetWorld(), PhasesConfig);

	PhaseManager->RegisterDynamicProcessor(*Processor);

	PhaseManager->Start(EntityManager->AsShared());

	return PhaseManager;
}

// Test using task sync manager to coordinate tick and mass
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FTaskSyncTest, FEngineTickTestBase, "System.Engine.Tick.TaskSyncTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ServerContext | EAutomationTestFlags::EngineFilter)
bool FTaskSyncTest::RunTest(const FString& Parameters)
{
	UE::Tick::FTaskSyncManager* SyncManager = UE::Tick::FTaskSyncManager::Get();
	if (!SyncManager)
	{
		// Skip test for now
		return true;
	}

	if (!CreateTestWorld())
	{
		return false;
	}

	float DeltaTime = 0.01f;
	int32 ActorCount = 1000;
	bool bSuccess = true;

	bSuccess &= CreateTestActors(ActorCount, AEngineTestTickActor::StaticClass());
	bSuccess &= BeginPlayInTestWorld();

	if (bSuccess)
	{
		check(TestActors.Num() == ActorCount);

		/** Add the test events */
		FName TestSourceName("EngineTestSource");

		FName TestTaskName("EngineTestTask");
		UE::Tick::FSyncPointDescription TestTaskDescription;
		TestTaskDescription.RegisteredName = TestTaskName;
		TestTaskDescription.SourceName = TestSourceName;
		TestTaskDescription.EventType = UE::Tick::ESyncPointEventType::GameThreadTask_HighPriority;
		TestTaskDescription.ActivationRules = UE::Tick::ESyncPointActivationRules::WaitForAllWork;
		TestTaskDescription.FirstPossibleTickGroup = TG_PrePhysics;
		TestTaskDescription.LastPossibleTickGroup = TG_PostPhysics;
		ensure(SyncManager->RegisterNewSyncPoint(TestTaskDescription));

		FName TestEventName("EngineTestEvent");
		UE::Tick::FSyncPointDescription TestEventDescription;
		TestEventDescription.RegisteredName = TestEventName;
		TestEventDescription.SourceName = TestSourceName;
		TestEventDescription.EventType = UE::Tick::ESyncPointEventType::SimpleEvent;
		TestEventDescription.ActivationRules = UE::Tick::ESyncPointActivationRules::WaitForTrigger;
		TestEventDescription.FirstPossibleTickGroup = TG_PrePhysics;
		TestEventDescription.LastPossibleTickGroup = TG_LastDemotable;
		ensure(SyncManager->RegisterNewSyncPoint(TestEventDescription));
		
		TickTestWorld(); // Tick once to create the sync tick functions

		// Test using requested work
		{
			UE::Tick::FSyncPointId TestTaskSyncPoint = SyncManager->FindSyncPoint(GetTestWorld(), TestTaskName);
			
			TArray<UE::Tick::FActiveSyncWorkHandle> WorkHandles;

			for (AEngineTestTickActor* TestActor : TestActors)
			{
				UE::Tick::FActiveSyncWorkHandle WorkHandle;
				SyncManager->RegisterWorkHandle(TestTaskSyncPoint, WorkHandle);

				// Disable normal tick
				TestActor->RegisterAllActorTickFunctions(false, false);
				
				ensure(WorkHandle.IsValid());

				WorkHandle.ReserveFutureWork(UE::Tick::ESyncWorkRepetition::Once);

				ensure(WorkHandle.RequestWork(&TestActor->PrimaryActorTick, UE::Tick::ESyncWorkRepetition::EveryFrame));
				WorkHandles.Add(MoveTemp(WorkHandle));
			}

			ResetTestActors();
			TickTestWorld();

			TickTestWorld();

			CheckTickCount(TEXT("RequestedWork"), 2);

			for (AEngineTestTickActor* TestActor : TestActors)
			{
				TestActor->RegisterAllActorTickFunctions(true, true);
			}
		}

		UE::Tick::FSyncPointId TestEventSyncPoint = SyncManager->FindSyncPoint(GetTestWorld(), TestEventName);
		FTickFunction* EventTickFunction = SyncManager->GetTickFunctionForSyncPoint(TestEventSyncPoint);
		check(EventTickFunction);

		for (AEngineTestTickActor* TestActor : TestActors)
		{
			// Add a tick dependency
			TestActor->PrimaryActorTick.AddPrerequisite(TestActor, *EventTickFunction);
		}

		// Use a message so it happens halfway through tick
		{
			auto PayloadTickFunction = [&](const FAsyncMessage& Message)
			{
				const FEngineTestTickPayload* Data = Message.GetPayloadData<const FEngineTestTickPayload>();
				if (Data)
				{
				
					FGraphEventRef FoundEvent;

					FGraphEventRef WaitEvent = FGraphEvent::CreateGraphEvent();

					UE::Tick::FTaskSyncResult Result = SyncManager->GetTaskGraphEvent(TestEventSyncPoint, FoundEvent);
					ensure(Result);
					ensure(FoundEvent.IsValid());

					Result = SyncManager->TriggerSyncPointAfterEvent(TestEventSyncPoint, WaitEvent);
					ensure(Result && Result.WasActivatedForFrame());

					WaitEvent->DispatchSubsequents(ENamedThreads::GameThread);

					Result = SyncManager->TriggerSyncPoint(TestEventSyncPoint);
					ensure(!Result && Result.WasActivatedForFrame());
				}
			};

		
			const FAsyncMessageId TickTestMessageId = { TickTestMessageTag };
			TSharedPtr<FAsyncMessageSystemBase> TestSystem = UAsyncMessageWorldSubsystem::GetSharedMessageSystem<FAsyncGameplayMessageSystem>(GetTestWorld());
			const FAsyncMessageHandle ListenerHandle = TestSystem->BindListener(TickTestMessageId, PayloadTickFunction);

			FEngineTestTickPayload PayloadData;
			FConstStructView PayloadView = FConstStructView::Make<FEngineTestTickPayload>(PayloadData);

			TestSystem->QueueMessageForBroadcast(TickTestMessageId, PayloadView);

			ResetTestActors();
			TickTestWorld();
			CheckTickCount(TEXT("MessageSync"), 1);

			TestSystem->UnbindListener(ListenerHandle);
		}

		// Test using mass to kickoff event
		{
			FMassEntityManager* EntityManager = UE::Mass::Utils::GetEntityManager(GetTestWorld());

			TArray<const UScriptStruct*> Fragments = { FEngineTestTickPayload::StaticStruct() };
			FMassArchetypeHandle TickArchetype = EntityManager->CreateArchetype(Fragments);

			UEngineTickTestProcessor* Processor = NewObject<UEngineTickTestProcessor>();
			Processor->CallInitialize(GetTransientPackage(), EntityManager->AsShared());
			Processor->EntityQuery.AddRequirement<FEngineTestTickPayload>(EMassFragmentAccess::ReadOnly);
			Processor->SyncPointName = TestEventName;


			TArray<FMassEntityHandle> TickEntities;
			EntityManager->BatchCreateEntities(TickArchetype, TestActors.Num(), TickEntities);
			TSharedRef<FMassProcessingPhaseManager> PhaseManager = InitializeMassProcessing(EntityManager, Processor);

			ResetTestActors();
			TickTestWorld();
			CheckTickCount(TEXT("MassSync"), 1);

			EntityManager->BatchDestroyEntities(TickEntities);

			PhaseManager->Deinitialize();
		}

		ensure(SyncManager->UnregisterSyncPoint(TestTaskDescription.RegisteredName, TestTaskDescription.SourceName));
		ensure(SyncManager->UnregisterSyncPoint(TestEventDescription.RegisteredName, TestEventDescription.SourceName));
		SyncManager->ReloadRegisteredData();
	}

	bSuccess &= DestroyTestWorld();

	return bSuccess && !ReportAnyErrors();
}

static TAutoConsoleVariable<int32> CVarEngineTickPerfOptions(
	TEXT("Automation.Test.EngineTickPerf.Options"),
	0,
	TEXT("Bitfield to modify options used for tick test.\n")
	TEXT("0 - No tick dependencies or intervals\n")
	TEXT("1 - Add tick dependencies\n")
	TEXT("2 - Add tick intervals\n")
	TEXT("3 - Add tick dependencies and intervals\n"),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarEngineTickPerfActorCount(
	TEXT("Automation.Test.EngineTickPerf.ActorCount"),
	1000,
	TEXT("Number of actors to spawn for tick test\n"),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarEngineTickPerfTickCount(
	TEXT("Automation.Test.EngineTickPerf.TickCount"),
	1000,
	TEXT("Number of frames to tick\n"),
	ECVF_Default);

static TAutoConsoleVariable<FString> CVarEngineTickPerfRunTests(
	TEXT("Automation.Test.EngineTickPerf.RunTests"),
	TEXT(""),
	TEXT("Specific tests to run separated by spaces. If empty it will run all the named tests that pass true to IsTestEnabled()\n"),
	ECVF_Default);

// Simplest task graph task to eliminate tick/safety overhead
class FEngineTestTickSimpleTask
{
public:
	AEngineTestTickActor* Actor = nullptr;
	ENamedThreads::Type DesiredThread = ENamedThreads::GameThread;

	FORCEINLINE FEngineTestTickSimpleTask(AEngineTestTickActor* InActor, ENamedThreads::Type InDesiredThread)
		: Actor(InActor)
		, DesiredThread(InDesiredThread)
	{
	}

	// Functions used by TGraphTask to schedule and execute task
	static FORCEINLINE TStatId GetStatId()
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FEngineTestTickSimpleTask, STATGROUP_TaskGraphTasks);
	}
	FORCEINLINE ENamedThreads::Type GetDesiredThread()
	{
		return DesiredThread;
	}
	static FORCEINLINE ESubsequentsMode::Type GetSubsequentsMode()
	{
		return ESubsequentsMode::TrackSubsequents;
	}
	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		Actor->VirtualTick();
	}
};

// Compares different ways of ticking actors for performance
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPerfTickTest, FEngineTickTestBase, "System.Engine.Tick.PerfTest", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ServerContext | EAutomationTestFlags::PerfFilter)
bool FPerfTickTest::RunTest(const FString& Parameters)
{
	const int32 ActorCount = CVarEngineTickPerfActorCount->GetInt();
	const int32 TickCount = CVarEngineTickPerfTickCount->GetInt();
	float DeltaTime = 0.01f;
	TArray<FString> TestsToRun;
	CVarEngineTickPerfRunTests->GetString().ParseIntoArray(TestsToRun, TEXT(" "));

	auto IsTestEnabled = [TestsToRun](FString InString, bool bEnableByDefault = true)
	{
		if (TestsToRun.Contains(InString) || (bEnableByDefault && TestsToRun.Num() == 0))
		{
			return true;
		}
		return false;
	};

	if (!CreateTestWorld())
	{
		ReportAnyErrors();
		return false;
	}

	if (BeginPlayInTestWorld())
	{
		UE_LOGF(LogStats, Log, "Running FPerfTickTest for %d actors over %d tick frames:", ActorCount, TickCount);

		{
			// Time to tick an empty world
			LOG_SCOPE_TIME(WorldBaseline);
			for (int32 i = 0; i < TickCount; i++)
			{
				TickTestWorld();
			}
		}

		if (!CreateTestActors(ActorCount, AEngineTestTickActor::StaticClass()))
		{
			return false;
		}

		const int32 TestOptions = CVarEngineTickPerfOptions->GetInt();
		const int32 RandomSeed = 0xABCD1234;
		FRandomStream RandomSource(RandomSeed);

		// Add some semi-random timing and dependency changes
		for (int32 i = 0; i < ActorCount; i++)
		{
			if ((TestOptions & 0x00000001) != 0 && i != (ActorCount-1))
			{
				// Enable dependencies on a random later actor
				TestActors[i]->AddTickPrerequisiteActor(TestActors[RandomSource.RandRange(i+1, ActorCount-1)]);
				// TestActors[i]->AddTickPrerequisiteActor(TestActors[RandomSource.RandHelper(ActorCount)]); // This creates infinite loops which can deadlock the engine
			}

			if ((TestOptions & 0x00000002) != 0)
			{
				// Enable a small interval, this should not affect actual timing
				TestActors[i]->SetActorTickInterval(DeltaTime / 2 + RandomSource.FRandRange(-DeltaTime/10, DeltaTime/10));
			}
		}

		if (IsTestEnabled(TEXT("WorldActorTick"), true))
		{
			ResetTestActors();
			{
				// Tick with normal task graph method
				LOG_SCOPE_TIME(WorldActorTick);
				for (int32 i = 0; i < TickCount; i++)
				{
					TickTestWorld();
				}
			}
			CheckTickCount(TEXT("WorldActorTick"), TickCount);
		}

		FSimpleMulticastDelegate LambdaDelegate, VirtualLambdaDelegate;
		FSimpleMulticastDelegate UObjectDelegate, VirtualUObjectDelegate;
		FSimpleMulticastDelegate WeakLambdaDelegate, VirtualWeakLambdaDelegate;

		for (AEngineTestTickActor* TestActor : TestActors)
		{
			// Unregister normal ticks
			TestActor->RegisterAllActorTickFunctions(false, false);

			// Check various delegate types, raw delegates are blocked on UObjects
			LambdaDelegate.AddLambda([TestActor]() {TestActor->DoTick(); });
			VirtualLambdaDelegate.AddLambda([TestActor]() {TestActor->VirtualTick(); });
			UObjectDelegate.AddUObject(TestActor, &AEngineTestTickActor::DoTick);
			VirtualUObjectDelegate.AddUObject(TestActor, &AEngineTestTickActor::VirtualTick);
			WeakLambdaDelegate.AddWeakLambda(TestActor, [TestActor]() {TestActor->DoTick(); });
			VirtualWeakLambdaDelegate.AddWeakLambda(TestActor, [TestActor]() {TestActor->VirtualTick(); });
		}

		if (IsTestEnabled(TEXT("TaskGraph"), false))
		{
			// Using task graph directly to avoid tick overhead, this is the same process the tick manager uses with construct and dispatch
			FGraphEventArray GraphEvents;
			GraphEvents.Reserve(TestActors.Num());

			ResetTestActors();
			{
				LOG_SCOPE_TIME(TaskGraphConstructAndWait);
				for (int32 i = 0; i < TickCount; i++)
				{
					for (AEngineTestTickActor* TestActor : TestActors)
					{
						TGraphTask<FEngineTestTickSimpleTask>* TaskPtr = TGraphTask<FEngineTestTickSimpleTask>::CreateTask(nullptr, ENamedThreads::GameThread).ConstructAndHold(TestActor, ENamedThreads::GameThread);
						GraphEvents.Emplace(TaskPtr->GetCompletionEvent());
					}

					// Separate loop as this is what tick does
					for (FGraphEventRef& GraphEventRef : GraphEvents)
					{
						GraphEventRef->Unlock();
					}

					FTaskGraphInterface::Get().ProcessUntilTasksComplete(GraphEvents, ENamedThreads::GameThread);
					GraphEvents.Reset();

					// Do an empty world tick to match baseline
					TickTestWorld();
				}
			}
			CheckTickCount(TEXT("TaskGraph"), TickCount);
		}

		if (IsTestEnabled(TEXT("BaseTask"), false))
		{
			// Use tasks api directly, but still on game thread. This is not a normal workflow but is useful for isolated profiling
			TArray<UE::Tasks::FTask> TaskArray;
			TaskArray.Reserve(TestActors.Num());

			ResetTestActors();
			{
				LOG_SCOPE_TIME(BaseTask);
				for (int32 i = 0; i < TickCount; i++)
				{
					for (AEngineTestTickActor* TestActor : TestActors)
					{
						TaskArray.Add(UE::Tasks::Launch(UE_SOURCE_LOCATION,
							[TestActor] {
								TestActor->VirtualTick();
							},
							UE::Tasks::ETaskPriority::Normal,
							UE::Tasks::EExtendedTaskPriority::GameThreadNormalPri
						));
					}

					UE::Tasks::FTask ReturnTask = UE::Tasks::Launch(UE_SOURCE_LOCATION,
						[] {},
						TaskArray,
						UE::Tasks::ETaskPriority::Normal,
						UE::Tasks::EExtendedTaskPriority::GameThreadNormalPri);

					ReturnTask.Wait();
					TaskArray.Reset();

					// Do an empty world tick to match baseline
					TickTestWorld();
				}
			}
			CheckTickCount(TEXT("BaseTask"), TickCount);
		}

		if (IsTestEnabled(TEXT("WorldTSTicker"), true))
		{
			FTSTicker TSTicker;
			for (AEngineTestTickActor* TestActor : TestActors)
			{
				TSTicker.AddTicker(FTickerDelegate::CreateWeakLambda(TestActor, [TestActor](float) {TestActor->VirtualTick(); return true; }), 0.0f);
			}

			ResetTestActors();
			{
				LOG_SCOPE_TIME(WorldTSTicker);
				for (int32 i = 0; i < TickCount; i++)
				{
					TickTestWorld();
					TSTicker.Tick(DeltaTime);
				}
			}
			CheckTickCount(TEXT("WorldTSTicker"), TickCount);
			TSTicker.Reset();
		}
		
		if (IsTestEnabled(TEXT("GameplayMessageSystem"), false))
		{
			const FAsyncMessageId TickTestMessageId = { TickTestMessageTag };
			auto PayloadTickFunction = [](const FAsyncMessage& Message)
			{
				const FEngineTestTickPayload* Data = Message.GetPayloadData<const FEngineTestTickPayload>();
				if (Data)
				{
					AEngineTestTickActor* TargetActor = Data->TargetActor.Get();
					if (ensure(TargetActor))
					{
						TargetActor->VirtualTick();
					}
				}
			};

			TSharedPtr<FAsyncGameplayMessageSystem> GameplaySystem = UAsyncMessageWorldSubsystem::GetSharedMessageSystem<FAsyncGameplayMessageSystem>(GetTestWorld());
			if (ensure(GameplaySystem))
			{
				const FAsyncMessageHandle ListenerHandle = GameplaySystem->BindListener(TickTestMessageId, PayloadTickFunction);

				FEngineTestTickPayload PayloadData;
				FConstStructView PayloadView = FConstStructView::Make<FEngineTestTickPayload>(PayloadData);

				ResetTestActors();
				{
					LOG_SCOPE_TIME(GameplayMessageSystem);
					for (int32 i = 0; i < TickCount; i++)
					{
						for (AEngineTestTickActor* TestActor : TestActors)
						{
							PayloadData.TargetActor = TestActor;
							GameplaySystem->QueueMessageForBroadcast(TickTestMessageId, PayloadView);
						}

						// This will run the tick group message tick handlers
						TickTestWorld();
					}
				}
				CheckTickCount(TEXT("GameplayMessageSystem"), TickCount);

				GameplaySystem->UnbindListener(ListenerHandle);
			}
		}

		if (IsTestEnabled(TEXT("MassProcessor"), false))
		{
			FMassEntityManager* EntityManager = UE::Mass::Utils::GetEntityManager(GetTestWorld());

			TArray<const UScriptStruct*> Fragments = { FEngineTestTickPayload::StaticStruct() };
			FMassArchetypeHandle TickArchetype = EntityManager->CreateArchetype(Fragments);

			UEngineTickTestProcessor* Processor = NewObject<UEngineTickTestProcessor>();
			Processor->CallInitialize(GetTransientPackage(), EntityManager->AsShared());
			Processor->EntityQuery.AddRequirement<FEngineTestTickPayload>(EMassFragmentAccess::ReadOnly);
			Processor->ForEachEntityChunkExecutionFunction = [](FMassExecutionContext& Context)
			{
				const TConstArrayView<FEngineTestTickPayload> TickPayloads = Context.GetFragmentView<FEngineTestTickPayload>();
				for (const FEngineTestTickPayload& Payload : TickPayloads)
				{
					AEngineTestTickActor* TargetActor = Payload.TargetActor.Get();
					if (ensure(TargetActor))
					{
						TargetActor->VirtualTick();
					}
				}
			};

			TArray<FMassEntityHandle> TickEntities;
			EntityManager->BatchCreateEntities(TickArchetype, TestActors.Num(), TickEntities);

			for (int32 i = 0; i < TestActors.Num(); i++)
			{
				check(TickEntities.IsValidIndex(i));

				// TODO more efficient way of initializing?
				FEngineTestTickPayload& Payload = EntityManager->GetFragmentDataChecked<FEngineTestTickPayload>(TickEntities[i]);
				Payload.TargetActor = TestActors[i];
			}

			TSharedRef<FMassProcessingPhaseManager> PhaseManager = InitializeMassProcessing(EntityManager, Processor);

			ResetTestActors();
			{
				LOG_SCOPE_TIME(MassProcessor);
				for (int32 i = 0; i < TickCount; i++)
				{
					// This will run the tick group message tick handlers
					TickTestWorld();
				}
			}
			CheckTickCount(TEXT("MassProcessor"), TickCount);

			EntityManager->BatchDestroyEntities(TickEntities);
			
			PhaseManager->Deinitialize();
		}

		UE::Tick::FTaskSyncManager* SyncManager = UE::Tick::FTaskSyncManager::Get();
		if (IsTestEnabled(TEXT("TaskSyncManager"), true) && SyncManager)
		{
			FName TestSourceName("EngineTestSource");

			FName TestTaskName("EngineTestTask");
			UE::Tick::FSyncPointDescription TestTaskDescription;
			TestTaskDescription.RegisteredName = TestTaskName;
			TestTaskDescription.SourceName = TestSourceName;
			TestTaskDescription.EventType = UE::Tick::ESyncPointEventType::GameThreadTask;
			TestTaskDescription.ActivationRules = UE::Tick::ESyncPointActivationRules::AlwaysActivate;
			TestTaskDescription.FirstPossibleTickGroup = TG_PrePhysics;
			TestTaskDescription.LastPossibleTickGroup = TG_PrePhysics;
			ensure(SyncManager->RegisterNewSyncPoint(TestTaskDescription));

			// Tick once to create events
			TickTestWorld();

			UE::Tick::FSyncPointId TestTaskSyncPoint = SyncManager->FindSyncPoint(GetTestWorld(), TestTaskName);

			TArray<UE::Tick::FActiveSyncWorkHandle> WorkHandles;

			for (AEngineTestTickActor* TestActor : TestActors)
			{
				UE::Tick::FActiveSyncWorkHandle WorkHandle;
				SyncManager->RegisterWorkHandle(TestTaskSyncPoint, WorkHandle);

				ensure(WorkHandle.RequestWork(&TestActor->PrimaryActorTick, UE::Tick::ESyncWorkRepetition::EveryFrame));
				WorkHandles.Add(MoveTemp(WorkHandle));
			}

			ResetTestActors();
			{
				LOG_SCOPE_TIME(TaskSyncManager);
				for (int32 i = 0; i < TickCount; i++)
				{
					TickTestWorld();
				}
			}
			CheckTickCount(TEXT("TaskSyncManager"), TickCount);

			WorkHandles.Reset();
			ensure(SyncManager->UnregisterSyncPoint(TestTaskDescription.RegisteredName, TestTaskDescription.SourceName));
			SyncManager->ReloadRegisteredData();
		}

		if (IsTestEnabled(TEXT("WorldTimerManager"), true))
		{
			FTimerManager& TimerManager = GetTestWorld()->GetTimerManager();
			TArray<FTimerHandle> TimerHandles;
			TimerHandles.Reserve(TestActors.Num());
			for (AEngineTestTickActor* TestActor : TestActors)
			{
				FTimerHandle& TimerHandle = TimerHandles.AddDefaulted_GetRef();
				TimerManager.SetTimer(TimerHandle, FTimerDelegate::CreateWeakLambda(TestActor, [TestActor]() {TestActor->VirtualTick(); }), 0.001f,
					FTimerManagerTimerParameters{ .bLoop = true, .bMaxOncePerFrame = true, .FirstDelay = 0.0f });
			}
		
			// Tick the world once as timers won't tick until the next frame even if they are initialized outside of tick
			TickTestWorld();

			ResetTestActors();
			{
				LOG_SCOPE_TIME(WorldTimerManager);
				for (int32 i = 0; i < TickCount; i++)
				{
					TickTestWorld();
				}
			}
			CheckTickCount(TEXT("WorldTimerManager"), TickCount);
			for (FTimerHandle& TimerHandle : TimerHandles)
			{
				TimerManager.ClearTimer(TimerHandle);
				ensure(!TimerHandle.IsValid());
			}
			TimerHandles.Empty();
		}

		if (IsTestEnabled(TEXT("WorldTickableFast"), true))
		{
			// Fastest possible TickableGameObject
			TArray<FEngineTestTickActorTickableFast> FastTickables;
			FastTickables.Reserve(TestActors.Num());
			for (AEngineTestTickActor* TestActor : TestActors)
			{
				FastTickables.Emplace(TestActor);
			}

			ResetTestActors();
			{
				LOG_SCOPE_TIME(WorldTickableFast);
				for (int32 i = 0; i < TickCount; i++)
				{
					TickTestWorld();
				}
			}
			CheckTickCount(TEXT("WorldTickableFast"), TickCount);
			FastTickables.Empty();
		}

		if (IsTestEnabled(TEXT("WorldTickableSlow"), false))
		{
			// Slower unoptimized TickableGameObject
			TArray<FEngineTestTickActorTickableSlow> SlowTickables;
			SlowTickables.Reserve(TestActors.Num());
			for (AEngineTestTickActor* TestActor : TestActors)
			{
				SlowTickables.Emplace(TestActor);
			}

			ResetTestActors();
			{
				LOG_SCOPE_TIME(WorldTickableSlow);
				for (int32 i = 0; i < TickCount; i++)
				{
					TickTestWorld();
				}
			}
			CheckTickCount(TEXT("WorldTickableSlow"), TickCount);
			SlowTickables.Empty();
		}



		if (IsTestEnabled(TEXT("LoopDoTick"), true))
		{
			// Raw function call tests, with a world tick before
			ResetTestActors();
			{
				LOG_SCOPE_TIME(LoopDoTick);
				for (int32 i = 0; i < TickCount; i++)
				{
					TickTestWorld();
					for (AEngineTestTickActor* TestActor : TestActors)
					{
						TestActor->DoTick();
					}
				}
			}
			CheckTickCount(TEXT("LoopDoTick"), TickCount);
		}

		if (IsTestEnabled(TEXT("LoopVirtualTick"), false))
		{
			ResetTestActors();
			{
				LOG_SCOPE_TIME(LoopVirtualTick);
				for (int32 i = 0; i < TickCount; i++)
				{
					TickTestWorld();
					for (AEngineTestTickActor* TestActor : TestActors)
					{
						TestActor->VirtualTick();
					}
				}
			}
			CheckTickCount(TEXT("LoopVirtualTick"), TickCount);
		}

		if (IsTestEnabled(TEXT("LoopExecuteTick"), true))
		{
			ResetTestActors();
			{
				FGraphEventRef FakeEvent;
				LOG_SCOPE_TIME(LoopExecuteTick);
				for (int32 i = 0; i < TickCount; i++)
				{
					TickTestWorld();
					for (AEngineTestTickActor* TestActor : TestActors)
					{
						// TODO could replace with registering a tick manager
						TestActor->PrimaryActorTick.ExecuteTick(DeltaTime, LEVELTICK_All, ENamedThreads::GameThread, FakeEvent);
					}

				}
			}
			CheckTickCount(TEXT("LoopExecuteTick"), TickCount);
		}

		if (IsTestEnabled(TEXT("LambdaDelegate"), true))
		{
			ResetTestActors();
			{
				LOG_SCOPE_TIME(LambdaDelegate);
				for (int32 i = 0; i < TickCount; i++)
				{
					TickTestWorld();
					LambdaDelegate.Broadcast();
				}
			}
			CheckTickCount(TEXT("LambdaDelegate"), TickCount);
			LambdaDelegate.Clear();
		}

		if (IsTestEnabled(TEXT("VirtualLambdaDelegate"), false))
		{
			ResetTestActors();
			{
				LOG_SCOPE_TIME(VirtualLambdaDelegate);
				for (int32 i = 0; i < TickCount; i++)
				{
					TickTestWorld();
					VirtualLambdaDelegate.Broadcast();
				}
			}
			CheckTickCount(TEXT("VirtualLambdaDelegate"), TickCount);
			VirtualLambdaDelegate.Clear();
		}

		if (IsTestEnabled(TEXT("UObjectDelegate"), false))
		{
			ResetTestActors();
			{
				LOG_SCOPE_TIME(UObjectDelegate);
				for (int32 i = 0; i < TickCount; i++)
				{
					TickTestWorld();
					UObjectDelegate.Broadcast();
				}
			}
			CheckTickCount(TEXT("UObjectDelegate"), TickCount);
			UObjectDelegate.Clear();
		}

		if (IsTestEnabled(TEXT("VirtualUObjectDelegate"), false))
		{
			ResetTestActors();
			{
				LOG_SCOPE_TIME(VirtualUObjectDelegate);
				for (int32 i = 0; i < TickCount; i++)
				{
					TickTestWorld();
					VirtualUObjectDelegate.Broadcast();
				}
			}
			CheckTickCount(TEXT("VirtualUObjectDelegate"), TickCount);
			VirtualUObjectDelegate.Clear();
		}

		if (IsTestEnabled(TEXT("WeakLambdaDelegate"), false))
		{
			ResetTestActors();
			{
				LOG_SCOPE_TIME(WeakLambdaDelegate);
				for (int32 i = 0; i < TickCount; i++)
				{
					TickTestWorld();
					WeakLambdaDelegate.Broadcast();
				}
			}
			CheckTickCount(TEXT("WeakLambdaDelegate"), TickCount);
			WeakLambdaDelegate.Clear();
		}

		if (IsTestEnabled(TEXT("VirtualWeakLambdaDelegate"), true))
		{
			ResetTestActors();
			{
				LOG_SCOPE_TIME(VirtualWeakLambdaDelegate);
				for (int32 i = 0; i < TickCount; i++)
				{
					TickTestWorld();
					VirtualWeakLambdaDelegate.Broadcast();
				}
			}
			CheckTickCount(TEXT("VirtualWeakLambdaDelegate"), TickCount);
			VirtualWeakLambdaDelegate.Clear();
		}
	}
	return DestroyTestWorld() && !ReportAnyErrors();
}

#endif // WITH_AUTOMATION_WORKER
