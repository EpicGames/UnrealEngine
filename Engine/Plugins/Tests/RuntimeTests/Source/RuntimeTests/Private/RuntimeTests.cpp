// Copyright Epic Games, Inc. All Rights Reserved.

#include "RuntimeTests.h"
#include "Modules/ModuleManager.h"
#include "ShaderComparisonTests.h"

#if WITH_AUTOMATION_TESTS

void FRuntimeTestsModule::StartupModule()
{
	// This code will execute after your module is loaded into memory; the exact timing is specified in the .uplugin file per-module
	FCompareBasepassShadersAutomationTestInstance = new FCompareBasepassShaders( TEXT("FCompareBasepassShaders") );
	check(FCompareBasepassShadersAutomationTestInstance != nullptr);
}

void FRuntimeTestsModule::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.
	delete FCompareBasepassShadersAutomationTestInstance;
}

#endif // WITH_AUTIOMATION_TESTS
	
IMPLEMENT_MODULE(FRuntimeTestsModule, RuntimeTests)
