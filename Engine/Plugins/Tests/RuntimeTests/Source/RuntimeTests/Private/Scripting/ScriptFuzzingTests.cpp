// Copyright Epic Games, Inc. All Rights Reserved.


#include "HAL/IConsoleManager.h"
#include "UObject/UObjectIterator.h"
#include "Engine/World.h"
#include "Trace/Trace.inl"
#include "UObject/Package.h"

UObject* CreateFuzzingHostObject(UWorld* World, UClass* DesiredClass, TArray<UObject*>& CreatedHosts)
{
	UObject* Result = nullptr;
	if (!DesiredClass->HasAnyClassFlags(CLASS_Abstract))
	{
		if (DesiredClass->IsChildOf(AActor::StaticClass()))
		{
			if (DesiredClass->HasAnyClassFlags(CLASS_NotPlaceable))
			{
				UE_LOGF(LogTemp, Warning, "Cannot fuzz non-static methods in %ls%ls as it is marked NotPlaceable", DesiredClass->GetPrefixCPP(), *DesiredClass->GetName());
			}
			else
			{
				FActorSpawnParameters SpawnParams;
				SpawnParams.bNoFail = true;
				SpawnParams.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
				Result = World->SpawnActor<AActor>(DesiredClass, FTransform::Identity, SpawnParams);
			}
		}
		else
		{
			UObject* Outer = GetTransientPackage();
			UClass* DesiredWithin = DesiredClass->ClassWithin;
			if (DesiredWithin == UObject::StaticClass())
			{
				DesiredWithin = nullptr;
			}

			if ((DesiredWithin == nullptr) && DesiredClass->IsChildOf(UActorComponent::StaticClass()))
			{
				DesiredWithin = AActor::StaticClass();
			}

			if ((DesiredWithin != nullptr) && !Outer->GetClass()->IsChildOf(DesiredWithin))
			{
				UE_LOGF(LogTemp, Log, "%ls%ls has a desired class within of %ls", DesiredClass->GetPrefixCPP(), *DesiredClass->GetName(), *DesiredWithin->GetName());
				Outer = CreateFuzzingHostObject(World, DesiredWithin, /*inout*/ CreatedHosts);
			}

			if (Outer != nullptr)
			{
				 Result = NewObject<UObject>(Outer, DesiredClass);
			}
			else
			{
				UE_LOGF(LogTemp, Warning, "Cannot fuzz non-static methods in %ls%ls, was unable to create an appropriate outer to satisify a ClassWithin constraint", DesiredClass->GetPrefixCPP(), *DesiredClass->GetName());
			}
		}
	}
	else
	{
		UE_LOGF(LogTemp, Warning, "Cannot fuzz non-static methods in %ls%ls as it is marked Abstract; we might be able to in derived classes", DesiredClass->GetPrefixCPP(), *DesiredClass->GetName());
	}

	if (Result != nullptr)
	{
		CreatedHosts.Add(Result);
	}

	return Result;
}


FAutoConsoleCommandWithWorldAndArgs GScriptFuzzingCommand(
	TEXT("Test.ScriptFuzzing"),
	TEXT("Fuzzes the script exposed API of engine classes"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(
		[](const TArray<FString>& Params, UWorld* World)
{
	FEditorScriptExecutionGuard AllowScriptExec;

	//@TODO: Ensures are still bad in BP callable methods, this is just to help get to the crashes first
	//@TODO: THis currently controls both ensures and checks, so it's no good
	//	TGuardValue<bool> DisableEnsures(GIgnoreDebugger, true);

	//@TODO: Need to fuzz these too
	TArray<FString> BannedPackageNames;
	// does lots of terrible things right now (UnrealEd) 
	BannedPackageNames.Add(TEXT("UnrealEd"));
	// slots have complicated lifecycle rules that aren't exposed programmatically (UMG)
	BannedPackageNames.Add(TEXT("UMG"));
	// DLL badness when not installed?  TBD
	BannedPackageNames.Add(TEXT("OculusHMD"));

	// Run thru all native classes
	for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
	{
		UClass* TestClass = *ClassIt;
		if (TestClass->HasAnyClassFlags(CLASS_Native))
		{
			// See if the class is in a banned package
			const FString OutermostPackageName = TestClass->GetOutermost()->GetName();
			bool bBanned = false;
			for (const FString& BannedPackageName : BannedPackageNames)
			{
				if (OutermostPackageName.Contains(BannedPackageName))
				{
					bBanned = true;
					break;
				}
			}
			if (bBanned)
			{
				continue;
			}

			// Determine if it has any script surface area
			bool bHasFunctions = false;
			bool bHasNonStaticFunctions = false;
			for (TFieldIterator<UFunction> FuncIt(TestClass); FuncIt; ++FuncIt)
			{
				UFunction* Function = *FuncIt;
				if (Function->HasAnyFunctionFlags(FUNC_BlueprintCallable | FUNC_BlueprintPure))
				{
					bHasFunctions = true;
					if (!Function->HasAnyFunctionFlags(FUNC_Static))
					{
						bHasNonStaticFunctions = true;
					}
				}
			}

			if (bHasFunctions)
			{
				// Create an instance of the object if necessary (function libraries can use the CDO instead)
				TArray<UObject*> CreatedHosts;
				UObject* CreatedInstance = nullptr;
				if (bHasNonStaticFunctions)
				{
					CreatedInstance = CreateFuzzingHostObject(World, TestClass, /*out*/ CreatedHosts);
				}

				int32 NumFunctionsWeFailedToFuzzDueToNoObject = 0;

				// Run thru all functions and fuzz them
				for (TFieldIterator<UFunction> FuncIt(TestClass); FuncIt; ++FuncIt)
				{
					UFunction* Function = *FuncIt;
					if (Function->HasAnyFunctionFlags(FUNC_BlueprintCallable | FUNC_BlueprintPure))
					{
						UObject* TestInstance = Function->HasAnyFunctionFlags(FUNC_Static) ? TestClass->GetDefaultObject() : CreatedInstance;
						
						if (TestInstance != nullptr)
						{
							UE_LOGF(LogTemp, Log, "Fuzzing %ls%ls::%ls() on %ls", TestClass->GetPrefixCPP(), *TestClass->GetName(), *Function->GetName(), *TestInstance->GetName());

							// Build a permutation matrix
							//@TODO: Do that, right now we're just testing all empty values

							FString CallStr = FString::Printf(TEXT("%s"), *Function->GetName());

							TestInstance->CallFunctionByNameWithArguments(*CallStr, *GLog, nullptr, /*bForceCallWithNonExec=*/ true);
						}
						else
						{
							if (!TestClass->HasAnyClassFlags(CLASS_Abstract))
							{
								++NumFunctionsWeFailedToFuzzDueToNoObject;
							}
						}
					}
				}

				if (NumFunctionsWeFailedToFuzzDueToNoObject > 0)
				{
					UE_LOGF(LogTemp, Warning, "Failed to fuzz %d non-static functions on %ls%ls because we could not make an object to test it on", NumFunctionsWeFailedToFuzzDueToNoObject, TestClass->GetPrefixCPP(), *TestClass->GetName());
				}

				// Tear down the created instance
				for (UObject* CreatedHost : CreatedHosts)
				{
					if (CreatedHost->IsA(AActor::StaticClass()))
					{
						World->DestroyActor(CastChecked<AActor>(CreatedHost));
					}
				}
			}
		}
	}
}));
