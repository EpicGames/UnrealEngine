// Copyright Epic Games, Inc. All Rights Reserved.

#include "ShaderComparisonTests.h"

#if WITH_AUTOMATION_TESTS

#include "Camera/CameraActor.h"
#include "Camera/CameraComponent.h"
#include "GameFramework/PlayerController.h"
#include "Engine/Engine.h"
#include "Kismet/GameplayStatics.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Tests/AutomationCommon.h"
#include "UnrealClient.h"

#if WITH_EDITOR
#include "ImageComparer.h"
#endif

#ifdef __APPLE__
#include "TargetConditionals.h"
#endif

#include <cfloat>
#include <memory>
#include <exception>
#include <stdexcept>

DEFINE_LOG_CATEGORY_STATIC(ShaderComparisonTests, Log, Log);

/**
 * Latent command to take a screenshot of the viewport
 */
DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FViewportScreenshotCommand, FString, ScreenshotFileName);

bool FViewportScreenshotCommand::Update()
{
    const bool bShowUI = false;
    const bool bAddFilenameSuffix = false;
    FScreenshotRequest::RequestScreenshot( ScreenshotFileName, bShowUI, bAddFilenameSuffix );
    UE_LOGF(ShaderComparisonTests, Log, "Taking Screenshot %ls.", *ScreenshotFileName);
    return true;
}

/**
 * Latent command to get performance numbers
 */
DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FGetAverageUnitTimes, std::shared_ptr<TArray<float>>, AverageTimes);

bool FGetAverageUnitTimes::Update()
{
    GEngine->GetAverageUnitTimes(*AverageTimes);
    return true;
}

/**
 * Latent command to set the camera
 */
DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FSetCamera, FTransform, Transform);

bool FSetCamera::Update()
{
    check(GEngine->GetWorldContexts().Num() == 1);
    const auto& WorldContext = GEngine->GetWorldContexts()[0];
    check(WorldContext.WorldType == EWorldType::Game);

    auto Camera = WorldContext.World()->SpawnActor<ACameraActor>();
    Camera->SetActorTransform(Transform);
    Camera->GetCameraComponent()->Activate();
    APlayerController* PlayerController = UGameplayStatics::GetPlayerController(WorldContext.World(), 0);
    if (!PlayerController)
    {
        UE_LOGF(ShaderComparisonTests, Error, "Unable to get PlayerController");
        return true;
    }
    PlayerController->SetViewTarget(Camera, FViewTargetTransitionParams());
    UE_LOGF(ShaderComparisonTests, Log, "Modifying camera.");
    return true;
}

/** 
 * Requests a enumeration of all shader vars to be tested
 */
void FCompareBasepassShaders::GetTests(TArray<FString>& OutBeautifiedNames, TArray <FString>& OutTestCommands) const
{
    OutBeautifiedNames.Add("FP16");
    OutTestCommands.Add("EXPERIMENTAL_FP16");
}

/** 
 * Execute the comparison for the given shader var
 *
 * @param Parameters - Should specify which shader var to test
 * @return    TRUE if deemed that the performance benefit outweighs the visual cost. FALSE otherwise.
 */
bool FCompareBasepassShaders::RunTest(const FString& Parameters)
{
// Mac platforms have some bugs that prevent some of the angles from working correctly
#if defined(__APPLE__)// && TARGET_OS_IPHONE
#define NUM_CAMERAS 11
    static FVector Translation[NUM_CAMERAS] = {
        FVector(5000.0f, 0.0f, 250.0f), FVector(4000.0f, 0.0f, 250.0f), FVector(3300.0f, 0.0f, 250.0f),
        FVector(3100.0f, 0.0f, 250.0f), FVector(1800.0f, 0.0f, 250.0f), FVector(1200.0f, 0.0f, 250.0f),
        FVector(500.0f, 0.0f, 250.0f), FVector(0.0f, 0.0f, 250.0f), FVector(-700.0f, -100.0f, 400.0f),
        FVector(-1600.0f, -100.0f, 400.0f), FVector(-4000.0f, 0.0f, 250.0f)
    };
    static FVector Rotation[NUM_CAMERAS] = {
        FVector(0.0f, 0.0f, 90.0f), FVector(0.0f, 0.0f, 90.0f), FVector(0.0f, 0.0f, 90.0f),
        FVector(0.0f, 0.0f, 90.0f), FVector(0.0f, 0.0f, 90.0f), FVector(0.0f, 0.0f, 90.0f),
        FVector(0.0f, 0.0f, 90.0f), FVector(0.0f, 0.0f, 90.0f), FVector(0.0f, 0.0f, 90.0f),
        FVector(0.0f, 0.0f, 90.0f), FVector(0.0f, 0.0f, 150.0f)
    };
#else
#define NUM_CAMERAS 15
    static FVector Translation[NUM_CAMERAS] = {
        FVector(5000.0f, 0.0f, 250.0f), FVector(4000.0f, 0.0f, 250.0f), FVector(3300.0f, 0.0f, 250.0f),
        FVector(3100.0f, 0.0f, 250.0f), FVector(1800.0f, 0.0f, 250.0f), FVector(1200.0f, 0.0f, 250.0f),
        FVector(500.0f, 0.0f, 250.0f), FVector(0.0f, 0.0f, 250.0f), FVector(-700.0f, -100.0f, 400.0f),
        FVector(-1600.0f, -100.0f, 400.0f), FVector(-2400.0f, -100.0f, 400.0f), FVector(-4000.0f, 0.0f, 250.0f),
        FVector(150.0f, -650.0f, 250.0f), FVector(1000.0f, -650.0f, 250.0f), FVector(1900.0f, -650.0f, 250.0f)
    };
    static FVector Rotation[NUM_CAMERAS] = {
        FVector(0.0f, 0.0f, 90.0f), FVector(0.0f, 0.0f, 90.0f), FVector(0.0f, 0.0f, 90.0f),
        FVector(0.0f, 0.0f, 90.0f), FVector(0.0f, 0.0f, 90.0f), FVector(0.0f, 0.0f, 90.0f),
        FVector(0.0f, 0.0f, 90.0f), FVector(0.0f, 0.0f, 90.0f), FVector(0.0f, 0.0f, 90.0f),
        FVector(0.0f, 0.0f, 90.0f), FVector(0.0f, 0.0f, 90.0f), FVector(0.0f, 0.0f, 150.0f),
        FVector(0.0f, 0.0f, 270.0f), FVector(0.0f, 0.0f, 270.0f), FVector(0.0f, 0.0f, 270.0f)
    };
#endif

    check(Parameters.Equals("EXPERIMENTAL_FP16"));
    if (!GEngine)
    {
        return false;
    }
    auto AverageTimes = std::make_shared<TArray<float>>();
    auto FP16AverageTimes = std::make_shared<TArray<float>>();
    AverageTimes->SetNum(4);
    FP16AverageTimes->SetNum(4);

    // TODO(mlentine): Fix this so we don't need separate take and load paths
    TArray<FString> ShaderNames = { FPaths::EngineDir() + "Shaders/Public/FP16Math.ush" };
    auto FileStrings = std::make_shared<TArray<FString>>();
    FileStrings->SetNum(ShaderNames.Num());
    FString MapName = "TM-ShaderModels";
    TArray<FString> ScreenshotFileName, FP16ScreenshotFileName;
    TArray<FString> RealScreenshotFileName, RealFP16ScreenshotFileName;
    ScreenshotFileName.SetNum(NUM_CAMERAS);
    RealScreenshotFileName.SetNum(NUM_CAMERAS);
    FP16ScreenshotFileName.SetNum(NUM_CAMERAS);
    RealFP16ScreenshotFileName.SetNum(NUM_CAMERAS);
    for (int i = 0; i < NUM_CAMERAS; ++i)
    {
        FString CompareBasepassShadersTestName = FString::Printf(TEXT("CompareBasepassShaders_Game/%s/%d"), *FPaths::GetBaseFilename(MapName), i);
		ScreenshotFileName[i] = AutomationCommon::GetScreenshotPath(CompareBasepassShadersTestName);
        CompareBasepassShadersTestName = FString::Printf(TEXT("Incoming/CompareBasepassShaders_Game/%s/%d"), *FPaths::GetBaseFilename(MapName), i);
		RealScreenshotFileName[i] = AutomationCommon::GetScreenshotPath(CompareBasepassShadersTestName);
        RealScreenshotFileName[i] = "../../../" + RealScreenshotFileName[i];
        CompareBasepassShadersTestName = FString::Printf(TEXT("CompareBasepassShaders_Game/%s_fp16/%d"), *FPaths::GetBaseFilename(MapName), i);
		FP16ScreenshotFileName[i] = AutomationCommon::GetScreenshotPath(CompareBasepassShadersTestName);
        CompareBasepassShadersTestName = FString::Printf(TEXT("Incoming/CompareBasepassShaders_Game/%s_fp16/%d"), *FPaths::GetBaseFilename(MapName), i);
		RealFP16ScreenshotFileName[i] = AutomationCommon::GetScreenshotPath(CompareBasepassShadersTestName);
        RealFP16ScreenshotFileName[i] = "../../../" + RealFP16ScreenshotFileName[i];
    }

    {
        AddCommand(new FLoadGameMapCommand(MapName));
        AddCommand(new FWaitForSpecifiedMapToLoadCommand(MapName));
        AddCommand(new FExecStringLatentCommand("recompileshaders changed"));
        AddCommand(new FWaitForShadersToFinishCompilingInGame());
    }

    {
        FTransform Transform(FQuat::MakeFromEuler(Rotation[NUM_CAMERAS - 1]), Translation[NUM_CAMERAS - 1]);
        AddCommand(new FSetCamera(Transform));
        AddCommand(new FEngineWaitLatentCommand(1.0f));
        AddCommand(new FExecStringLatentCommand("stat unit"));
        AddCommand(new FEngineWaitLatentCommand(5.0f));
        AddCommand(new FGetAverageUnitTimes(AverageTimes));
        AddCommand(new FEngineWaitLatentCommand(5.0f));
        AddCommand(new FGetAverageUnitTimes(AverageTimes));
        AddCommand(new FExecStringLatentCommand("stat unit"));
    }
    for (int i = 0; i < NUM_CAMERAS; ++i)
    {
        FTransform Transform(FQuat::MakeFromEuler(Rotation[i]), Translation[i]);
        AddCommand(new FSetCamera(Transform));
        AddCommand(new FEngineWaitLatentCommand(1.0f));
        AddCommand(new FViewportScreenshotCommand(ScreenshotFileName[i]));
        AddCommand(new FEngineWaitLatentCommand(1.5f));
    }

    AddCommand(new FDelayedFunctionLatentCommand([=] {
        FString str;
        FString DefineStr = FString("#define ") + Parameters + " 1\n";
        for (int i = 0; i < ShaderNames.Num(); ++i) {
            FFileHelper::LoadFileToString((*FileStrings)[i], *ShaderNames[i]);
            str = DefineStr + (*FileStrings)[i];
            FFileHelper::SaveStringToFile(str, *ShaderNames[i]);
            UE_LOGF(ShaderComparisonTests, Log, "Modifying shader %ls.", *ShaderNames[i]);
        }
    }));
    
    {
        AddCommand(new FExecStringLatentCommand("recompileshaders changed"));
        AddCommand(new FWaitForShadersToFinishCompilingInGame());
    }

    {
        FTransform Transform(FQuat::MakeFromEuler(Rotation[NUM_CAMERAS - 1]), Translation[NUM_CAMERAS - 1]);
        AddCommand(new FSetCamera(Transform));
        AddCommand(new FEngineWaitLatentCommand(1.0f));
        AddCommand(new FExecStringLatentCommand("stat unit"));
        AddCommand(new FEngineWaitLatentCommand(5.0f));
        AddCommand(new FGetAverageUnitTimes(FP16AverageTimes));
        AddCommand(new FEngineWaitLatentCommand(5.0f));
        AddCommand(new FGetAverageUnitTimes(FP16AverageTimes));
        AddCommand(new FExecStringLatentCommand("stat unit"));
    }
    for (int i = 0; i < NUM_CAMERAS; ++i)
    {
        FTransform Transform(FQuat::MakeFromEuler(Rotation[i]), Translation[i]);
        AddCommand(new FSetCamera(Transform));
        AddCommand(new FEngineWaitLatentCommand(1.0f));
        AddCommand(new FViewportScreenshotCommand(FP16ScreenshotFileName[i]));
        AddCommand(new FEngineWaitLatentCommand(1.5f));
    }

    AddCommand(new FDelayedFunctionLatentCommand([=] {
        for (int i = 0; i < ShaderNames.Num(); ++i) {
            FFileHelper::SaveStringToFile((*FileStrings)[i], *ShaderNames[i]);
        }
    }));

    // TODO(mlentine): Find appropriate normalized scores
    AddCommand(new FDelayedFunctionLatentCommand([=] {

        float similarity = FLT_MAX;
        for (int i = 0; i < NUM_CAMERAS; ++i)
        {
            FString PathName = FPaths::AutomationDir() + FString::Printf(TEXT("Incoming/CompareBasepassShaders_Game/%s_delta/%d"), *FPaths::GetBaseFilename(MapName), i) / FPlatformProperties::PlatformName();
            PathName = PathName + TEXT("_") + AutomationCommon::GetRenderDetailsString();
            FPaths::MakePathRelativeTo(PathName, *FPaths::RootDir());
            PathName = "../../../" + PathName;
            UE_LOGF(ShaderComparisonTests, Log, "Screenshots are at %ls and %ls.", *RealScreenshotFileName[i], *RealFP16ScreenshotFileName[i]);
            UE_LOGF(ShaderComparisonTests, Log, "Difference is stored in %ls", *PathName);
#if WITH_EDITOR
            auto single_similarity = FImageComparer().CompareStructuralSimilarity(RealScreenshotFileName[i], RealFP16ScreenshotFileName[i], FImageComparer::EStructuralSimilarityComponent::Luminance, "");
            similarity = single_similarity < similarity ? single_similarity : similarity;
            UE_LOGF(ShaderComparisonTests, Log, "Similarity is %f after %d.", similarity, i);
#endif 
        }
    
        UE_LOGF(ShaderComparisonTests, Log, "FP16 ran in %f, with FP32 in %f.", (*FP16AverageTimes)[2], (*AverageTimes)[2]);
        if ((*FP16AverageTimes)[2] > (*AverageTimes)[2] + 0.2)
        {
            UE_LOGF(ShaderComparisonTests, Error, "FP16 is slower than FP32!");
        }
        float PerfDelta = ((*AverageTimes)[2] - (*FP16AverageTimes)[2])/10 + 1;
        float ImageDelta = (1 - similarity) / 2.f;
        UE_LOGF(ShaderComparisonTests, Log, "Perf delta is %f and Image delta is %f.", PerfDelta, ImageDelta);
        if (ImageDelta / PerfDelta > 0.04)
        {
            UE_LOGF(ShaderComparisonTests, Error, "The quality detriment doesn't outweigh the performance.");
        }
    }));

    // This is a bit pointless right now
    return true;
}

#endif // WITH_AUTOMATION_TESTS