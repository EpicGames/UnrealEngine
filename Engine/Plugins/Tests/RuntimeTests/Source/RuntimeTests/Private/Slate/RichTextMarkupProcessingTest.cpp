// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Framework/Text/ITextDecorator.h"

#include "Framework/Text/RichTextMarkupProcessing.h"

#define LOCTEXT_NAMESPACE "Slate.RichText.MarkupProcessingTest"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRTFProcessingTest, "System.Slate.RichText.MarkupProcessing", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)

namespace
{
	bool CompareTextRange(const FTextRange& ExpectedRange, const FTextRange& ActualRange)
	{
		return ExpectedRange.BeginIndex == ActualRange.BeginIndex && ExpectedRange.EndIndex == ActualRange.EndIndex;
	}

	bool CompareMetaData(const TMap<FString, FTextRange>& ExpectedMetaData, const TMap<FString, FTextRange>& ActualMetaData)
	{
		if(ExpectedMetaData.Num() != ActualMetaData.Num())
		{
			return false;
		}

		const int32 MetaDataCount = ExpectedMetaData.Num();
		TMap<FString, FTextRange>::TConstIterator ExpectedIterator = ExpectedMetaData.CreateConstIterator();
		TMap<FString, FTextRange>::TConstIterator ActualIterator = ActualMetaData.CreateConstIterator();
		for(int32 i = 0; i < MetaDataCount; ++i, ++ExpectedIterator, ++ActualIterator)
		{
			if( ExpectedIterator.Key() != ActualIterator.Key() || !( CompareTextRange( ExpectedIterator.Value(), ActualIterator.Value() ) ) )
			{
				return false;
			}
		}

		return true;
	}

	bool CompareRunResults(const FTextRunParseResults& ExpectedResults, const FTextRunParseResults& ActualResults)
	{
		return
			ExpectedResults.Name == ActualResults.Name &&
			CompareTextRange(ExpectedResults.OriginalRange, ActualResults.OriginalRange) &&
			CompareTextRange(ExpectedResults.ContentRange, ActualResults.ContentRange) &&
			CompareMetaData(ExpectedResults.MetaData, ActualResults.MetaData);
	}

	bool CompareRunResultsArray(const TArray<FTextRunParseResults>& ExpectedResultsArray, const TArray<FTextRunParseResults>& ActualResultsArray)
	{
		if( ExpectedResultsArray.Num() != ActualResultsArray.Num() )
		{
			return false;
		}

		const int32 ResultCount = ExpectedResultsArray.Num();
		for(int32 i = 0; i < ResultCount; ++i)
		{
			if( !( CompareRunResults(ExpectedResultsArray[i], ActualResultsArray[i]) ) )
			{
				return false;
			}
		}

		return true;
	}

	bool CompareLineResults(const FTextLineParseResults& ExpectedResults, const FTextLineParseResults& ActualResults)
	{
		return
			CompareTextRange(ExpectedResults.Range, ActualResults.Range) &&
			CompareRunResultsArray(ExpectedResults.Runs, ActualResults.Runs);
	}

	bool CompareLineResultsArray(const TArray<FTextLineParseResults>& ExpectedResultsArray, const TArray<FTextLineParseResults>& ActualResultsArray)
	{
		if( ExpectedResultsArray.Num() != ActualResultsArray.Num() )
		{
			return false;
		}

		const int32 ResultCount = ExpectedResultsArray.Num();
		for(int32 i = 0; i < ResultCount; ++i)
		{
			if( !( CompareLineResults(ExpectedResultsArray[i], ActualResultsArray[i]) ) )
			{
				return false;
			}
		}

		return true;
	}
}

bool FRTFProcessingTest::RunTest (const FString& Parameters)
{
#if UE_ENABLE_ICU
	TSharedRef< FDefaultRichTextMarkupParser > RichTextMarkupProcessing = FDefaultRichTextMarkupParser::GetStaticInstance();

	// Text
	{
		FString Input = TEXT("Plain text");
		TArray<FTextLineParseResults> ExpectedResultsArray;
		{
			FTextLineParseResults ExpectedLineParseResults(FTextRange(0, 10));
			{
				FTextRunParseResults ExpectedRunParseResults(TEXT(""), FTextRange(0, 10));
				ExpectedLineParseResults.Runs.Add(ExpectedRunParseResults);
			}
			ExpectedResultsArray.Add(ExpectedLineParseResults);
		}
		FString ExpectedOutput = TEXT("Plain text");

		TArray<FTextLineParseResults> ActualResultsArray;
		FString ActualOutput;
		RichTextMarkupProcessing->Process(ActualResultsArray, Input, ActualOutput);

		if( ActualOutput != ExpectedOutput || !(CompareLineResultsArray(ExpectedResultsArray, ActualResultsArray)) )
		{
			AddError(TEXT("Output and/or results for RTF processing do not match expectations."));
		}
	}

	// Text with Escape Sequences
	{
		FString Input = TEXT("&quot;&gt;&lt;&amp;");
		TArray<FTextLineParseResults> ExpectedResultsArray;
		{
			FTextLineParseResults ExpectedLineParseResults(FTextRange(0, 4));
			{
				FTextRunParseResults ExpectedRunParseResults(TEXT(""), FTextRange(0, 4));
				ExpectedLineParseResults.Runs.Add(ExpectedRunParseResults);
			}
			ExpectedResultsArray.Add(ExpectedLineParseResults);
		}
		FString ExpectedOutput = TEXT("\"><&");

		TArray<FTextLineParseResults> ActualResultsArray;
		FString ActualOutput;
		RichTextMarkupProcessing->Process(ActualResultsArray, Input, ActualOutput);

		if( ActualOutput != ExpectedOutput || !(CompareLineResultsArray(ExpectedResultsArray, ActualResultsArray)) )
		{
			AddError(TEXT("Output and/or results for RTF processing do not match expectations."));
		}
	}

	// Element
	{
		FString Input = TEXT("<Name/>");
		TArray<FTextLineParseResults> ExpectedResultsArray;
		{
			FTextLineParseResults ExpectedLineParseResults(FTextRange(0, 7));
			{
				FTextRunParseResults ExpectedRunParseResults(TEXT("Name"), FTextRange(0, 7));
				ExpectedLineParseResults.Runs.Add(ExpectedRunParseResults);
			}
			ExpectedResultsArray.Add(ExpectedLineParseResults);
		}
		FString ExpectedOutput = TEXT("<Name/>");

		TArray<FTextLineParseResults> ActualResultsArray;
		FString ActualOutput;
		RichTextMarkupProcessing->Process(ActualResultsArray, Input, ActualOutput);

		if( ActualOutput != ExpectedOutput || !(CompareLineResultsArray(ExpectedResultsArray, ActualResultsArray)) )
		{
			AddError(TEXT("Output and/or results for RTF processing do not match expectations."));
		}
	}

	// Element with Content
	{
		FString Input = TEXT("<Name>Content</>");
		TArray<FTextLineParseResults> ExpectedResultsArray;
		{
			FTextLineParseResults ExpectedLineParseResults(FTextRange(0, 16));
			{
				FTextRunParseResults ExpectedRunParseResults(TEXT("Name"), FTextRange(0, 16), FTextRange(6, 13));
				ExpectedLineParseResults.Runs.Add(ExpectedRunParseResults);
			}
			ExpectedResultsArray.Add(ExpectedLineParseResults);
		}
		FString ExpectedOutput = TEXT("<Name>Content</>");

		TArray<FTextLineParseResults> ActualResultsArray;
		FString ActualOutput;
		RichTextMarkupProcessing->Process(ActualResultsArray, Input, ActualOutput);

		if( ActualOutput != ExpectedOutput || !(CompareLineResultsArray(ExpectedResultsArray, ActualResultsArray)) )
		{
			AddError(TEXT("Output and/or results for RTF processing do not match expectations."));
		}
	}

	// Element with Content with Escape Sequences
	{
		FString Input = TEXT("<Name>&lt;Content&gt;</>");
		TArray<FTextLineParseResults> ExpectedResultsArray;
		{
			FTextLineParseResults ExpectedLineParseResults(FTextRange(0, 18));
			{
				FTextRunParseResults ExpectedRunParseResults(TEXT("Name"), FTextRange(0, 18), FTextRange(6, 15));
				ExpectedLineParseResults.Runs.Add(ExpectedRunParseResults);
			}
			ExpectedResultsArray.Add(ExpectedLineParseResults);
		}
		FString ExpectedOutput = TEXT("<Name><Content></>");

		TArray<FTextLineParseResults> ActualResultsArray;
		FString ActualOutput;
		RichTextMarkupProcessing->Process(ActualResultsArray, Input, ActualOutput);

		if( ActualOutput != ExpectedOutput || !(CompareLineResultsArray(ExpectedResultsArray, ActualResultsArray)) )
		{
			AddError(TEXT("Output and/or results for RTF processing do not match expectations."));
		}
	}

	// Element with Single Attribute
	{
		FString Input = TEXT("<Name AttKey=\"AttValue\"/>");
		TArray<FTextLineParseResults> ExpectedResultsArray;
		{
			FTextLineParseResults ExpectedLineParseResults(FTextRange(0, 25));
			{
				FTextRunParseResults ExpectedRunParseResults(TEXT("Name"), FTextRange(0, 25));
				ExpectedRunParseResults.MetaData.Add(TEXT("AttKey"), FTextRange(14, 22));
				ExpectedLineParseResults.Runs.Add(ExpectedRunParseResults);
			}
			ExpectedResultsArray.Add(ExpectedLineParseResults);
		}
		FString ExpectedOutput = TEXT("<Name AttKey=\"AttValue\"/>");

		TArray<FTextLineParseResults> ActualResultsArray;
		FString ActualOutput;
		RichTextMarkupProcessing->Process(ActualResultsArray, Input, ActualOutput);

		if( ActualOutput != ExpectedOutput || !(CompareLineResultsArray(ExpectedResultsArray, ActualResultsArray)) )
		{
			AddError(TEXT("Output and/or results for RTF processing do not match expectations."));
		}
	}

	// Element with Single Attribute with Escape Sequences
	{
		FString Input = TEXT("<Name AttKey=\"&quot;AttValue&quot;\"/>");
		TArray<FTextLineParseResults> ExpectedResultsArray;
		{
			FTextLineParseResults ExpectedLineParseResults(FTextRange(0, 27));
			{
				FTextRunParseResults ExpectedRunParseResults(TEXT("Name"), FTextRange(0, 27));
				ExpectedRunParseResults.MetaData.Add(TEXT("AttKey"), FTextRange(14, 24));
				ExpectedLineParseResults.Runs.Add(ExpectedRunParseResults);
			}
			ExpectedResultsArray.Add(ExpectedLineParseResults);
		}
		FString ExpectedOutput = TEXT("<Name AttKey=\"\"AttValue\"\"/>");

		TArray<FTextLineParseResults> ActualResultsArray;
		FString ActualOutput;
		RichTextMarkupProcessing->Process(ActualResultsArray, Input, ActualOutput);

		if( ActualOutput != ExpectedOutput || !(CompareLineResultsArray(ExpectedResultsArray, ActualResultsArray)) )
		{
			AddError(TEXT("Output and/or results for RTF processing do not match expectations."));
		}
	}

	// Element with Single Attribute and Content
	{
		FString Input = TEXT("<Name AttKey=\"AttValue\">Content</>");
		TArray<FTextLineParseResults> ExpectedResultsArray;
		{
			FTextLineParseResults ExpectedLineParseResults(FTextRange(0, 34));
			{
				FTextRunParseResults ExpectedRunParseResults(TEXT("Name"), FTextRange(0, 34), FTextRange(24, 31));
				ExpectedRunParseResults.MetaData.Add(TEXT("AttKey"), FTextRange(14, 22));
				ExpectedLineParseResults.Runs.Add(ExpectedRunParseResults);
			}
			ExpectedResultsArray.Add(ExpectedLineParseResults);
		}
		FString ExpectedOutput = TEXT("<Name AttKey=\"AttValue\">Content</>");

		TArray<FTextLineParseResults> ActualResultsArray;
		FString ActualOutput;
		RichTextMarkupProcessing->Process(ActualResultsArray, Input, ActualOutput);

		if( ActualOutput != ExpectedOutput || !(CompareLineResultsArray(ExpectedResultsArray, ActualResultsArray)) )
		{
			AddError(TEXT("Output and/or results for RTF processing do not match expectations."));
		}
	}

	// Element with Multiple Attributes
	{
		FString Input = TEXT("<Name AttKey0=\"AttValue0\" AttKey1=\"AttValue1\"/>");
		TArray<FTextLineParseResults> ExpectedResultsArray;
		{
			FTextLineParseResults ExpectedLineParseResults(FTextRange(0, 47));
			{
				FTextRunParseResults ExpectedRunParseResults(TEXT("Name"), FTextRange(0, 47));
				ExpectedRunParseResults.MetaData.Add(TEXT("AttKey0"), FTextRange(15, 24));
				ExpectedRunParseResults.MetaData.Add(TEXT("AttKey1"), FTextRange(35, 44));
				ExpectedLineParseResults.Runs.Add(ExpectedRunParseResults);
			}
			ExpectedResultsArray.Add(ExpectedLineParseResults);
		}
		FString ExpectedOutput = TEXT("<Name AttKey0=\"AttValue0\" AttKey1=\"AttValue1\"/>");

		TArray<FTextLineParseResults> ActualResultsArray;
		FString ActualOutput;
		RichTextMarkupProcessing->Process(ActualResultsArray, Input, ActualOutput);

		if( ActualOutput != ExpectedOutput || !(CompareLineResultsArray(ExpectedResultsArray, ActualResultsArray)) )
		{
			AddError(TEXT("Output and/or results for RTF processing do not match expectations."));
		}
	}

	// Element with Multiple Attributes and Content
	{
		FString Input = TEXT("<Name AttKey0=\"AttValue0\" AttKey1=\"AttValue1\">Content</>");
		TArray<FTextLineParseResults> ExpectedResultsArray;
		{
			FTextLineParseResults ExpectedLineParseResults(FTextRange(0, 56));
			{
				FTextRunParseResults ExpectedRunParseResults(TEXT("Name"), FTextRange(0, 56), FTextRange(46, 53));
				ExpectedRunParseResults.MetaData.Add(TEXT("AttKey0"), FTextRange(15, 24));
				ExpectedRunParseResults.MetaData.Add(TEXT("AttKey1"), FTextRange(35, 44));
				ExpectedLineParseResults.Runs.Add(ExpectedRunParseResults);
			}
			ExpectedResultsArray.Add(ExpectedLineParseResults);
		}
		FString ExpectedOutput = TEXT("<Name AttKey0=\"AttValue0\" AttKey1=\"AttValue1\">Content</>");

		TArray<FTextLineParseResults> ActualResultsArray;
		FString ActualOutput;
		RichTextMarkupProcessing->Process(ActualResultsArray, Input, ActualOutput);

		if( ActualOutput != ExpectedOutput || !(CompareLineResultsArray(ExpectedResultsArray, ActualResultsArray)) )
		{
			AddError(TEXT("Output and/or results for RTF processing do not match expectations."));
		}
	}

	// Element with Multiple Attributes with Escape Sequences and Content with Escape Sequences
	{
		FString Input = TEXT("<Name AttKey0=\"&quot;AttValue0&quot;\" AttKey1=\"&quot;AttValue1&quot;\">&lt;Content&gt;</>");
		TArray<FTextLineParseResults> ExpectedResultsArray;
		{
			FTextLineParseResults ExpectedLineParseResults(FTextRange(0, 62));
			{
				FTextRunParseResults ExpectedRunParseResults(TEXT("Name"), FTextRange(0, 62), FTextRange(50, 59));
				ExpectedRunParseResults.MetaData.Add(TEXT("AttKey0"), FTextRange(15, 26));
				ExpectedRunParseResults.MetaData.Add(TEXT("AttKey1"), FTextRange(37, 48));
				ExpectedLineParseResults.Runs.Add(ExpectedRunParseResults);
			}
			ExpectedResultsArray.Add(ExpectedLineParseResults);
		}
		FString ExpectedOutput = TEXT("<Name AttKey0=\"\"AttValue0\"\" AttKey1=\"\"AttValue1\"\"><Content></>");

		TArray<FTextLineParseResults> ActualResultsArray;
		FString ActualOutput;
		RichTextMarkupProcessing->Process(ActualResultsArray, Input, ActualOutput);

		if( ActualOutput != ExpectedOutput || !(CompareLineResultsArray(ExpectedResultsArray, ActualResultsArray)) )
		{
			AddError(TEXT("Output and/or results for RTF processing do not match expectations."));
		}
	}
#else
	AddWarning(TEXT("Rich text format markup parsing requires regular expression support - regular expression support is not available without ICU - test disabled."));
#endif
	return true;
}

#undef LOCTEXT_NAMESPACE 
