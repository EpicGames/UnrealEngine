// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once 

#include "GameFramework/Actor.h"
#include "Misc/AutomationTest.h"
#include "Tests/AutomationCommon.h"
#include "MassProcessor.h"
#include "MassEntityTypes.h"
#include "MassEntityQuery.h"
#include "EngineRuntimeTests.generated.h"

#define UE_API RUNTIMETESTS_API

class UBillboardComponent;

/** A simple actor class that can be manually ticked to test for correctness and performance */
UCLASS(MinimalAPI)
class AEngineTestTickActor : public AActor
{
	GENERATED_BODY()

public:
	UE_API AEngineTestTickActor(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	/** Number of times this has ticked since reset */
	UPROPERTY(BlueprintReadOnly, Category=Default)
	int32 TickCount;
	
	/** Indicates when this was ticked in a frame, with 1 being first */
	UPROPERTY(BlueprintReadOnly, Category = Default)
	int32 TickOrder;

	/** Used to set TickOrder, reset to 1 at the start of every frame */
	static UE_API int32 CurrentTickOrder;

	/** If it should actually increase tick count */
	UPROPERTY(BlueprintReadOnly, Category = Default)
	bool bShouldIncrementTickCount;

	/** If it should perform other busy work */
	UPROPERTY(BlueprintReadOnly, Category=Default)
	bool bShouldDoMath;

	/** Used for bShouldDoMath */
	UPROPERTY(BlueprintReadOnly, Category=Default)
	float MathCounter;

	/** Used for bShouldDoMath */
	UPROPERTY(BlueprintReadOnly, Category=Default)
	float MathIncrement;

	/** Used for bShouldDoMath */
	UPROPERTY(BlueprintReadOnly, Category=Default)
	float MathLimit;

	/** Reset state before next test, call this after unregistering tick */
	UE_API virtual void ResetState();

	/** Do the actual work */
	UE_API void DoTick();

	/** Virtual function wrapper */
	UE_API virtual void VirtualTick();

	/** AActor version */
	UE_API virtual void Tick(float DeltaSeconds) override;

private:
	UPROPERTY()
	TObjectPtr<UBillboardComponent> SpriteComponent;
};

USTRUCT(NotBlueprintType, meta=(Hidden))
struct FEngineTestTickPayload : public FMassFragment
{
	GENERATED_BODY()
	
	TWeakObjectPtr<AEngineTestTickActor> TargetActor = nullptr;
};

// Copied from MassEntityTestSuite 
UCLASS(MinimalAPI)
class UEngineTickTestProcessor : public UMassProcessor
{
	GENERATED_BODY()
public:
	UE_API UEngineTickTestProcessor();
	FMassProcessorExecutionOrder& GetMutableExecutionOrder() { return ExecutionOrder; }
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;
	UE_API virtual FGraphEventRef DispatchProcessorTasks(const TSharedPtr<FMassEntityManager>& EntityManager, FMassExecutionContext& ExecutionContext, const FGraphEventArray& Prerequisites = FGraphEventArray()) override;

	virtual bool ShouldAllowQueryBasedPruning(const bool bRuntimeMode = true) const { return false; }
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override {}

	using FExecutionFunction = TFunction<void(FMassEntityManager& EntityManager, FMassExecutionContext& Context)>;
	FExecutionFunction ExecutionFunction;

	/** 
	 * By default ExecutionFunction is configured to pass this function over to EntityQuery.ForEachEntityChunk call. 
	 * Note that this function won't be used if you override ExecutionFunction's default value.
	 */
	FMassExecuteFunction ForEachEntityChunkExecutionFunction;

	/** public on purpose, this is a test processor, no worries about access*/
	FMassEntityQuery EntityQuery;

	/** SyncPoint event to emit after processor task has finished */
	FName SyncPointName;
};


#if WITH_AUTOMATION_WORKER

/** Automation test base class that wraps a test world and handles checking tick counts */
class FEngineTickTestBase : public FAutomationTestBase
{
public:
	UE_API FEngineTickTestBase(const FString& InName, const bool bInComplexTask);

	UE_API virtual ~FEngineTickTestBase() override;

	/** Gets the world being tested */
	UE_API UWorld* GetTestWorld() const;

	/** Creates a world where actors can be spawned */
	UE_API virtual bool CreateTestWorld();

	/** Spawn actors of subclass */
	UE_API virtual bool CreateTestActors(int32 ActorCount, TSubclassOf<AEngineTestTickActor> ActorClass);

	/** Start play in world, prepare for ticking */
	UE_API virtual bool BeginPlayInTestWorld();

	/** Tick one frame in test world */
	UE_API virtual bool TickTestWorld(float DeltaTime = 0.01f);

	/** Reset the test */
	UE_API virtual bool ResetTestActors();

	/** Checks TickCount on every actor */
	UE_API virtual bool CheckTickCount(const TCHAR* TickTestName, int32 TickCount);

	/** Destroys the test actors */
	UE_API virtual bool DestroyAllTestActors();
	
	/** Destroys the test world */
	UE_API virtual bool DestroyTestWorld();

	/** Reports errors to automation system, returns true if there were errors */
	UE_API virtual bool ReportAnyErrors();

protected:
	TUniquePtr<FTestWorldWrapper> WorldWrapper;
	TArray<AEngineTestTickActor*> TestActors;
};

#endif // WITH_AUTOMATION_WORKER

#undef UE_API
