// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Misc/AutomationTest.h"

class FCompareBasepassShaders;

class FRuntimeTestsModule : public IModuleInterface
{
#if WITH_AUTOMATION_TESTS

public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	FCompareBasepassShaders* FCompareBasepassShadersAutomationTestInstance;

#endif // WITH_AUTOMATION_TESTS
};
