// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tests/AutomationCommon.h"

/**
 * CompareAllShaderVars
 * Comparison automation test the determines which shader variables need extra precision
 */
#if WITH_AUTOMATION_TESTS
IMPLEMENT_COMPLEX_AUTOMATION_CLASS(FCompareBasepassShaders, "System.Engine.CompareShaderPrecision", EAutomationTestFlags::ClientContext | EAutomationTestFlags::EngineFilter | EAutomationTestFlags::Disabled)
#endif
