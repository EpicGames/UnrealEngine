// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class RuntimeTests : ModuleRules
{
	public RuntimeTests(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"MassCore",
				"MassEntity",
			}
		);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore",
				"AsyncMessageSystem",
				"GameplayTags",
			}
		);

		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.Add("ScreenShotComparisonTools");
		}
	}
}
